// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class MeshDescriptionOperations : ModuleRules
    {
        public MeshDescriptionOperations(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"MeshDescription",
                }
            );

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
                    "StaticMeshDescription",
				}
			);

            AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");
        }
    }
}
