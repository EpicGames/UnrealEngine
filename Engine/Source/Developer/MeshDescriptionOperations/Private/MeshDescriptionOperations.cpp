// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescriptionOperations.h"

#include "MeshDescription.h"
#include "Misc/SecureHash.h"
#include "StaticMeshOperations.h"

#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FDefaultModuleImpl, MeshDescriptionOperations)

//////////////////////////////////////////////////////////////////////////
// Converters

void FMeshDescriptionOperations::ConvertHardEdgesToSmoothGroup(const FMeshDescription& SourceMeshDescription, TArray<uint32>& FaceSmoothingMasks)
{
	FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(SourceMeshDescription, FaceSmoothingMasks);
}

void FMeshDescriptionOperations::ConvertSmoothGroupToHardEdges(const TArray<uint32>& FaceSmoothingMasks, FMeshDescription& DestinationMeshDescription)
{
	FStaticMeshOperations::ConvertSmoothGroupToHardEdges(FaceSmoothingMasks, DestinationMeshDescription);
}

void FMeshDescriptionOperations::ConvertToRawMesh(const FMeshDescription& SourceMeshDescription, FRawMesh& DestinationRawMesh, const TMap<FName, int32>& MaterialMap)
{
	FStaticMeshOperations::ConvertToRawMesh(SourceMeshDescription, DestinationRawMesh, MaterialMap);
}

void FMeshDescriptionOperations::ConvertFromRawMesh(const FRawMesh& SourceRawMesh, FMeshDescription& DestinationMeshDescription, const TMap<int32, FName>& MaterialMap)
{
	FStaticMeshOperations::ConvertFromRawMesh(SourceRawMesh, DestinationMeshDescription, MaterialMap);
}

void FMeshDescriptionOperations::AppendMeshDescription(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, FAppendSettings& AppendSettings)
{
	FStaticMeshOperations::AppendMeshDescription(SourceMesh, TargetMesh, AppendSettings);
}

//////////////////////////////////////////////////////////////////////////
// Normals tangents and Bi-normals

EComputeNTBsFlags ConvertTangentOptionsToNTBsFlags(FMeshDescriptionOperations::ETangentOptions TangentOptions)
{
	EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::None;
	ComputeNTBsOptions |= (TangentOptions & FMeshDescriptionOperations::BlendOverlappingNormals) ? EComputeNTBsFlags::BlendOverlappingNormals : EComputeNTBsFlags::None;
	ComputeNTBsOptions |= (TangentOptions & FMeshDescriptionOperations::IgnoreDegenerateTriangles) ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;
	ComputeNTBsOptions |= (TangentOptions & FMeshDescriptionOperations::UseMikkTSpace) ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;
	ComputeNTBsOptions |= (TangentOptions & FMeshDescriptionOperations::UseWeightedAreaAndAngle) ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;

	return ComputeNTBsOptions;
}

void FMeshDescriptionOperations::RecomputeNormalsAndTangentsIfNeeded(FMeshDescription& MeshDescription, ETangentOptions TangentOptions, bool bForceRecomputeNormals, bool bForceRecomputeTangents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionOperations::RecomputeNormalsAndTangentsIfNeeded);

	EComputeNTBsFlags ComputeNTBsOptions = ConvertTangentOptionsToNTBsFlags(TangentOptions);
	ComputeNTBsOptions |= (bForceRecomputeNormals) ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
	ComputeNTBsOptions |= (bForceRecomputeTangents) ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;


	FStaticMeshOperations::RecomputeNormalsAndTangentsIfNeeded(MeshDescription, ComputeNTBsOptions);
}

void FMeshDescriptionOperations::CreatePolygonNTB(FMeshDescription& MeshDescription, float ComparisonThreshold)
{
	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription, ComparisonThreshold);
}

void FMeshDescriptionOperations::CreateNormals(FMeshDescription& MeshDescription, FMeshDescriptionOperations::ETangentOptions TangentOptions, bool bComputeTangent)
{
	EComputeNTBsFlags ComputeNTBsOptions = ConvertTangentOptionsToNTBsFlags(TangentOptions);

	//We only check the tangents if MikkTSpace is not enabled, disabling MikkTSpace when bComputeTangent is true keeps the same behavior as previously
	ComputeNTBsOptions &= (bComputeTangent) ? ~EComputeNTBsFlags::UseMikkTSpace : ~EComputeNTBsFlags::None;

	FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, ComputeNTBsOptions);
}

void FMeshDescriptionOperations::CreateMikktTangents(FMeshDescription& MeshDescription, FMeshDescriptionOperations::ETangentOptions TangentOptions)
{
	bool bIgnoreDegenerate = (TangentOptions & FMeshDescriptionOperations::IgnoreDegenerateTriangles) != 0;
	FStaticMeshOperations::ComputeMikktTangents(MeshDescription, bIgnoreDegenerate);
}

void FMeshDescriptionOperations::FindOverlappingCorners(FOverlappingCorners& OutOverlappingCorners, const FMeshDescription& MeshDescription, float ComparisonThreshold)
{
	FStaticMeshOperations::FindOverlappingCorners(OutOverlappingCorners, MeshDescription, ComparisonThreshold);
}

int32 FMeshDescriptionOperations::GetUVChartCount(FMeshDescription& MeshDescription, int32 SrcLightmapIndex, ELightmapUVVersion LightmapUVVersion, const FOverlappingCorners& OverlappingCorners)
{
	return FStaticMeshOperations::GetUVChartCount(MeshDescription, SrcLightmapIndex, LightmapUVVersion, OverlappingCorners);
}

bool FMeshDescriptionOperations::CreateLightMapUVLayout(FMeshDescription& MeshDescription,
	int32 SrcLightmapIndex,
	int32 DstLightmapIndex,
	int32 MinLightmapResolution,
	ELightmapUVVersion LightmapUVVersion,
	const FOverlappingCorners& OverlappingCorners)
{
	return FStaticMeshOperations::CreateLightMapUVLayout(MeshDescription, SrcLightmapIndex, DstLightmapIndex, MinLightmapResolution, LightmapUVVersion, OverlappingCorners);
}

bool FMeshDescriptionOperations::GenerateUniqueUVsForStaticMesh(const FMeshDescription& MeshDescription, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2D>& OutTexCoords)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDescriptionOperations::GenerateUniqueUVsForStaticMesh);

	FStaticMeshOperations::FGenerateUVOptions GenerateUVOptions;
	GenerateUVOptions.TextureResolution = TextureResolution;
	GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes = bMergeIdenticalMaterials;
	GenerateUVOptions.UVMethod = FStaticMeshOperations::EGenerateUVMethod::Legacy;

	return FStaticMeshOperations::GenerateUV(MeshDescription, GenerateUVOptions, OutTexCoords);
}

bool FMeshDescriptionOperations::AddUVChannel(FMeshDescription& MeshDescription)
{
	return FStaticMeshOperations::AddUVChannel(MeshDescription);
}

bool FMeshDescriptionOperations::InsertUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex)
{
	return FStaticMeshOperations::InsertUVChannel(MeshDescription, UVChannelIndex);
}

bool FMeshDescriptionOperations::RemoveUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex)
{
	return FStaticMeshOperations::RemoveUVChannel(MeshDescription, UVChannelIndex);
}

void FMeshDescriptionOperations::GeneratePlanarUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords)
{
	FStaticMeshOperations::GeneratePlanarUV(MeshDescription, Params, OutTexCoords);
}

void FMeshDescriptionOperations::GenerateCylindricalUV(FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords)
{
	FStaticMeshOperations::GenerateCylindricalUV(MeshDescription, Params, OutTexCoords);
}

void FMeshDescriptionOperations::GenerateBoxUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords)
{
	FStaticMeshOperations::GenerateBoxUV(MeshDescription, Params, OutTexCoords);
}

void FMeshDescriptionOperations::RemapPolygonGroups(FMeshDescription& MeshDescription, TMap<FPolygonGroupID, FPolygonGroupID>& Remap)
{
	MeshDescription.RemapPolygonGroups(Remap);
}

void FMeshDescriptionOperations::SwapPolygonPolygonGroup(FMeshDescription& MeshDescription, int32 SectionIndex, int32 TriangleIndexStart, int32 TriangleIndexEnd, bool bRemoveEmptyPolygonGroup)
{
	FStaticMeshOperations::SwapPolygonPolygonGroup(MeshDescription, SectionIndex, TriangleIndexStart, TriangleIndexEnd, bRemoveEmptyPolygonGroup);
}

bool FMeshDescriptionOperations::HasVertexColor(const FMeshDescription& MeshDescription)
{
	return FStaticMeshOperations::HasVertexColor(MeshDescription);
}

void FMeshDescriptionOperations::BuildWeldedVertexIDRemap(const FMeshDescription& MeshDescription, const float WeldingThreshold, TMap<FVertexID, FVertexID>& OutVertexIDRemap)
{
	FStaticMeshOperations::BuildWeldedVertexIDRemap(MeshDescription, WeldingThreshold, OutVertexIDRemap);
}

FSHAHash FMeshDescriptionOperations::ComputeSHAHash(const FMeshDescription& MeshDescription)
{
	return FStaticMeshOperations::ComputeSHAHash(MeshDescription);
}
