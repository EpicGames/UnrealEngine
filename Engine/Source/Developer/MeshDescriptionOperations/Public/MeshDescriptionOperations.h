// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Misc/SecureHash.h"
#include "StaticMeshOperations.h"

class FName;
struct FMeshDescription;
struct FOverlappingCorners;
struct FRawMesh;
struct FUVMapParameters;

enum class ELightmapUVVersion : int32;
struct FPolygonGroupID;
struct FVertexID;
struct FVertexInstanceID;


//////////////////////////////////////////////////////////////////////////
// FMeshDescriptionOperations has been deprecated, most of it's features have been moved over FStaticMeshOperations.
class MESHDESCRIPTIONOPERATIONS_API FMeshDescriptionOperations
{
public:
	enum ETangentOptions
	{
		None = 0,
		BlendOverlappingNormals = 0x00000001,
		IgnoreDegenerateTriangles = 0x00000002,
		UseMikkTSpace = 0x00000004,
		UseWeightedAreaAndAngle = 0x00000008, //Use surface area and angle as a ratio when computing normals
	};

	typedef FStaticMeshOperations::FAppendSettings FAppendSettings;

	/** Convert this mesh description into the old FRawMesh format. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::ConvertToRawMesh() instead.")
	static void ConvertToRawMesh(const FMeshDescription& SourceMeshDescription, FRawMesh& DestinationRawMesh, const TMap<FName, int32>& MaterialMap);

	/** Convert old FRawMesh format to MeshDescription. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::ConvertFromRawMesh() instead.")
	static void ConvertFromRawMesh(const FRawMesh& SourceRawMesh, FMeshDescription& DestinationMeshDescription, const TMap<int32, FName>& MaterialMap);

	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::AppendMeshDescription() instead.")
	static void AppendMeshDescription(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, FAppendSettings& AppendSettings);

	/*
	 * Check if all normals and tangents are valid, if not recompute them
	 */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::RecomputeNormalsAndTangentsIfNeeded() instead.")
	static void RecomputeNormalsAndTangentsIfNeeded(FMeshDescription& MeshDescription, ETangentOptions TangentOptions, bool bForceRecomputeNormals = false, bool bForceRecomputeTangents = false);

	/**
	 * Compute normal, tangent and Bi-Normal for every polygon in the mesh description. (this do not compute Vertex NTBs)
	 * It also remove the degenerated polygon from the mesh description
	 */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::CreatComputePolygonTangentsAndNormals() instead.")
	static void CreatePolygonNTB(FMeshDescription& MeshDescription, float ComparisonThreshold);

	/** Compute normal, tangent and Bi-Normal(only if bComputeTangent is true) for every vertex in the mesh description. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::CreateNormals() instead.")
	static void CreateNormals(FMeshDescription& MeshDescription, ETangentOptions TangentOptions, bool bComputeTangent);

	/** Compute tangent and Bi-Normal using mikkt space for every vertex in the mesh description. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::CreateMikktTangents() instead.")
	static void CreateMikktTangents(FMeshDescription& MeshDescription, ETangentOptions TangentOptions);

	/** Find all overlapping vertex using the threshold in the mesh description. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::FindOverlappingCorners() instead.")
	static void FindOverlappingCorners(FOverlappingCorners& OverlappingCorners, const FMeshDescription& MeshDescription, float ComparisonThreshold);
	
	/** Find all charts in the mesh description. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::GetUVChartCount() instead.")
	static int32 GetUVChartCount(FMeshDescription& MeshDescription, int32 SrcLightmapIndex, ELightmapUVVersion LightmapUVVersion, const FOverlappingCorners& OverlappingCorners);

	/**
	 * Find and pack UV charts for lightmap.
	 * The packing algorithm uses a rasterization method, hence the resolution parameter.
	 *
	 * If the given minimum resolution is not enough to handle all the charts, generation will fail.
	 *
	 * @param MeshDescription        Edited mesh
	 * @param SrcLightmapIndex       index of the source UV channel
	 * @param DstLightmapIndex       index of the destination UV channel
	 * @param MinLightmapResolution  Minimum resolution used for the packing
	 * @param LightmapUVVersion      Algorithm version
	 * @param OverlappingCorners     Overlapping corners of the given mesh
	 * @return                       UV layout correctly generated
	 */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::CreateLightMapUVLayout() instead.")
	static bool CreateLightMapUVLayout(FMeshDescription& MeshDescription,
		int32 SrcLightmapIndex,
		int32 DstLightmapIndex,
		int32 MinLightmapResolution,
		ELightmapUVVersion LightmapUVVersion,
		const FOverlappingCorners& OverlappingCorners);

	/** Create some UVs from the specified mesh description data. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::GenerateUniqueUVsForStaticMesh() instead.")
	static bool GenerateUniqueUVsForStaticMesh(const FMeshDescription& MeshDescription, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2D>& OutTexCoords);

	/** Add a UV channel to the MeshDescription. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::AddUVChannel() instead.")
	static bool AddUVChannel(FMeshDescription& MeshDescription);

	/** Insert a UV channel at the given index to the MeshDescription. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::InsertUVChannel() instead.")
	static bool InsertUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex);

	/** Remove the UV channel at the given index from the MeshDescription. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::RemoveUVChannel() instead.")
	static bool RemoveUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex);

	/** Generate planar UV mapping for the MeshDescription */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::GeneratePlanarUV() instead.")
	static void GeneratePlanarUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords);

	/** Generate cylindrical UV mapping for the MeshDescription */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::GenerateCylindricalUV() instead.")
	static void GenerateCylindricalUV(FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords);

	/** Generate box UV mapping for the MeshDescription */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::GenerateBoxUV() instead.")
	static void GenerateBoxUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords);

	UE_DEPRECATED(4.25, "Use FMeshDescription::RemapPolygonGroups() instead.")
	static void RemapPolygonGroups(FMeshDescription& MeshDescription, TMap<FPolygonGroupID, FPolygonGroupID>& Remap);

	/*
	 * Move some polygon to a new PolygonGroup(section)
	 * SectionIndex: The target section we want to assign the polygon. See bRemoveEmptyPolygonGroup to know how its used
	 * TriangleIndexStart: The triangle index is compute as follow: foreach polygon {TriangleIndex += Polygon->NumberTriangles}
	 * TriangleIndexEnd: The triangle index is compute as follow: foreach polygon {TriangleIndex += Polygon->NumberTriangles}
	 * bRemoveEmptyPolygonGroup: If true, any polygonGroup that is empty after moving a polygon will be delete.
	 *                           This parameter impact how SectionIndex is use
	 *                           If param is true  : PolygonGroupTargetID.GetValue() do not necessary equal SectionIndex in case there is less sections then SectionIndex
	 *                           If param is false : PolygonGroupTargetID.GetValue() equal SectionIndex, we will add all necessary missing PolygonGroupID (this can generate empty PolygonGroupID)
	 */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::SwapPolygonPolygonGroup() instead.")
	static void SwapPolygonPolygonGroup(FMeshDescription& MeshDescription, int32 SectionIndex, int32 TriangleIndexStart, int32 TriangleIndexEnd, bool bRemoveEmptyPolygonGroup);

	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::ConvertHardEdgesToSmoothGroup() instead.")
	static void ConvertHardEdgesToSmoothGroup(const FMeshDescription& SourceMeshDescription, TArray<uint32>& FaceSmoothingMasks);

	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::ConvertSmoothGroupToHardEdges() instead.")
	static void ConvertSmoothGroupToHardEdges(const TArray<uint32>& FaceSmoothingMasks, FMeshDescription& DestinationMeshDescription);

	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::HasVertexColor() instead.")
	static bool HasVertexColor(const FMeshDescription& MeshDescription);

	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::BuildWeldedVertexIDRemap() instead.")
	static void BuildWeldedVertexIDRemap(const FMeshDescription& MeshDescription, const float WeldingThreshold, TMap<FVertexID, FVertexID>& OutVertexIDRemap);

	/** Computes the SHA hash of all the attributes values in the MeshDescription. */
	UE_DEPRECATED(4.25, "Use FStaticMeshOperations::ComputeSHAHash() instead.")
	static FSHAHash ComputeSHAHash(const FMeshDescription& MeshDescription);
};
